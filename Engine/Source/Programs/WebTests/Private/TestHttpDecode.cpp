// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"

#include <catch2/catch_test_macros.hpp>

#include "GenericPlatform/GenericPlatformHttp.h"

#define HTTP_SERVER_SUITE_TAGS "[HTTPServer]"
#define HTTP_SERVER_TEST_CASE(x, ...) TEST_CASE(x, HTTP_SERVER_SUITE_TAGS)

// Helper to generate a 2GB Test String
FString Generate2GBURL()
{
	FString TwoGBURL;
	constexpr int32 Size{ MAX_int32 - 1 };
	TwoGBURL.Reserve(Size);

	for (int32 i = 0; i < (Size - 2); ++i)
	{
		TwoGBURL.AppendChar('A');
	}

	TwoGBURL.AppendChar('%');
	return TwoGBURL;
}

// To verify UE-172890
HTTP_SERVER_TEST_CASE("URL Decoding")
{
	/** This test checks whether this method tries to read beyond the bounds on the input encoded string if it ends with '%'. This is because,
	in an encoded string, the '%' character is used as an escape character to represent special characters or bytes that are not allowed or have a special
	meaning in the context of the URL, and so indicates following characters in the encoded string. We want to make sure these methods do not attempt to
	read that expected next character when there is none and thus read out of bounds. **/
//	SECTION("Verify that FGenericPlatformHttp::UrlDecode() does not encounter out-of-bounds read error if encoded string ends in '%'")
//	{
//		// Create "encoded string" ending in '%'
//		FString encodedString = "test%";
//
//#if !PLATFORM_WINDOWS
//		FGenericPlatformHttp::UrlDecode(encodedString);
//		SUCCEED();
//#else
//		CHECK_NOTHROW(FGenericPlatformHttp::UrlDecode(encodedString),
//			"Verify there is no out - of - bounds read error when decoding the URL");
//#endif //!PLATFORM_WINDOWS
//	}

//	// Test disabled by default, should only be ran on Win64 where 2GB+ strings are supported
//	SECTION("FGenericPlatformHttp::UrlDecode() does not encounter a signedness error when decoding a 2GB URL with '%' at the end")
//	{
//#if !PLATFORM_WINDOWS
//		SUCCEED("This section is only supported on windows.");
//#else
//		// Create 2 GB URL ending with "%u"
//		FString twoGBURL = Generate2GBURL();
//
//		CHECK_NOTHROW(FGenericPlatformHttp::UrlDecode(twoGBURL),
//			"Verify no signedness error that leads to out-of-bounds read error");
//#endif //!PLATFORM_WINDOWS
//	}
}

