// Copyright Epic Games, Inc. All Rights Reserved.
#include "CoreMinimal.h"
#include "Containers/BackgroundableTicker.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformProcess.h"
#include "Http.h"
#include "Misc/CommandLine.h"
#include "WebSocketsLog.h"
#include "WebSocketsModule.h"
#include "IWebSocket.h"
#include "TestHarness.h"
#include "WebTestFixtures.h"

/**
 *  WebSockets Tests
 *  -----------------------------------------------------------------------------------------------
 *
 *  PURPOSE:
 *
 *	Integration Tests to make sure all kinds of WebSockets client features in C++ work well on different platforms,
 *  including but not limited to error handing, retrying, threading, SSL and profiling.
 * 
 *  -----------------------------------------------------------------------------------------------
 */

#define WEBSOCKETS_TAG "[WebSockets]"

class FWebSocketsModuleTestFixture
{
public:
	FWebSocketsModuleTestFixture()
		: OldVerbosity(LogWebSockets.GetVerbosity())
	{
		CVarHttpInsecureProtocolEnabled->Set(true);

		if (Settings.bVeryVerbose)
		{
			LogWebSockets.SetVerbosity(ELogVerbosity::VeryVerbose);
		}

		WebSocketsModule = new FWebSocketsModule();
		IModuleInterface* Module = WebSocketsModule;
		Module->StartupModule();
	}

	virtual ~FWebSocketsModuleTestFixture()
	{
		if (WebSocketsModule != nullptr)
		{
			IModuleInterface* WebSocketsModuleInterface = WebSocketsModule;
			WebSocketsModuleInterface->ShutdownModule();
			delete WebSocketsModuleInterface;
		}

		if (OldVerbosity != LogWebSockets.GetVerbosity())
		{
			LogWebSockets.SetVerbosity(OldVerbosity);
		}
	}

	void DisableWarningsInThisTest()
	{
		if (!Settings.bVeryVerbose)
		{
			LogWebSockets.SetVerbosity(ELogVerbosity::Error);
		}
	}

	const FString UrlWithInvalidPortToTestConnectTimeout() const { return FString::Format(TEXT("ws://{0}:{1}"), { *Settings.WebServerIp, 8765 }); }
	const FString UrlWebSocketsTests() const { return Settings.UrlWebSocketsTests(); }

	FWebTestSettings Settings;
	FWebSocketsModule* WebSocketsModule;
	ELogVerbosity::Type OldVerbosity;
};

class FRunUntilQuitRequestedFixture : public FWebSocketsModuleTestFixture
{
public:
	FRunUntilQuitRequestedFixture()
	{
	}

	~FRunUntilQuitRequestedFixture()
	{
		TickRunner.RunUntilQuitRequested();
	}

	void RequestQuit()          { TickRunner.bQuitRequested = true; }
	void SetSucceeded()         { TickRunner.bSucceeded = true; }
	bool HasSucceeded() const   { return TickRunner.bSucceeded; }

	FWebTestTickRunner TickRunner;
};

TEST_CASE_METHOD(FRunUntilQuitRequestedFixture, "WebSockets can connect then send and receive message", WEBSOCKETS_TAG)
{
	TSharedPtr<IWebSocket> WebSocket = WebSocketsModule->CreateWebSocket(FString::Format(TEXT("{0}/echo/"), { *UrlWebSocketsTests() }));

	WebSocket->OnConnected().AddLambda([this, WebSocket](){
		WebSocket->Send(TEXT("hi websockets tests"));
	});

	WebSocket->OnMessage().AddLambda([this, WebSocket](const FString& MessageString) {
		CHECK(MessageString == TEXT("hi websockets tests"));
		WebSocket->Close();
		SetSucceeded();
	});

	WebSocket->OnClosed().AddLambda([this, WebSocket](int32 /* StatusCode */, const FString& /* Reason */, bool /* bWasClean */){
		CHECK(HasSucceeded());
		RequestQuit();
	});

	WebSocket->OnConnectionError().AddLambda([this, WebSocket](const FString& /* Error */){
		CHECK(false);
		RequestQuit();
	});

	WebSocket->Connect();
}

TEST_CASE_METHOD(FRunUntilQuitRequestedFixture, "WebSockets can be closed by server", WEBSOCKETS_TAG)
{
	TSharedPtr<IWebSocket> WebSocket = WebSocketsModule->CreateWebSocket(FString::Format(TEXT("{0}/close_on_receive_message/"), { *UrlWebSocketsTests() }));

	WebSocket->OnConnected().AddLambda([this, WebSocket](){
		WebSocket->Send(TEXT("close me"));
	});

	WebSocket->OnClosed().AddLambda([this, WebSocket](int32 StatusCode, const FString& /* Reason */, bool /* bWasClean */){
		CHECK(StatusCode == 1000); // 1000 means default normal closure
		RequestQuit();
	});

	WebSocket->OnConnectionError().AddLambda([this, WebSocket](const FString& /* Error */){
		CHECK(false);
		RequestQuit();
	});

	WebSocket->Connect();
}

TEST_CASE_METHOD(FRunUntilQuitRequestedFixture, "WebSockets module can shut down when there are still websockets connections", WEBSOCKETS_TAG)
{
	DisableWarningsInThisTest();

	TSharedPtr<IWebSocket> WebSocket = WebSocketsModule->CreateWebSocket(FString::Format(TEXT("{0}/echo/"), { *UrlWebSocketsTests() }));

	WebSocket->OnConnected().AddLambda([this, WebSocket](){
		RequestQuit();
	});

	WebSocket->OnConnectionError().AddLambda([this, WebSocket](const FString& /* Error */){
		CHECK(false);
		RequestQuit();
	});

	WebSocket->Connect();
}

class FConnectWhenShutdownFixture : public FRunUntilQuitRequestedFixture
{
public:
	FConnectWhenShutdownFixture()
	{
		WebSocket = WebSocketsModule->CreateWebSocket(FString::Format(TEXT("{0}/echo/"), { *UrlWebSocketsTests() }));
	}

	~FConnectWhenShutdownFixture()
	{
		// Waiting until closed, then the Connect call can actually launch without early return
		TickRunner.RunUntilQuitRequested();

		WebSocket->Connect();
	}

	TSharedPtr<IWebSocket> WebSocket;
};

TEST_CASE_METHOD(FConnectWhenShutdownFixture, "WebSockets can call connect when shutdown", WEBSOCKETS_TAG)
{
	DisableWarningsInThisTest();

	WebSocket->OnConnected().AddLambda([this](){
		WebSocket->Send(TEXT("hi websockets tests"));
	});

	WebSocket->OnMessage().AddLambda([this](const FString& MessageString) {
		CHECK(MessageString == TEXT("hi websockets tests"));
		WebSocket->Close();
		SetSucceeded();
	});

	WebSocket->OnClosed().AddLambda([this](int32 /* StatusCode */, const FString& /* Reason */, bool /* bWasClean */){
		CHECK(HasSucceeded());
		RequestQuit();
	});

	WebSocket->OnConnectionError().AddLambda([this](const FString& /* Error */){
		CHECK(false);
		RequestQuit();
	});

	WebSocket->Connect();
}

// Fixture that starts FWebSocketsModule first, then FHttpModule after, verifying the two can coexist
// without double-initializing or double-finalizing the underlying SSL library.
class FHttpAndWebSocketsFixture : public FRunUntilQuitRequestedFixture
{
public:
	FHttpAndWebSocketsFixture()
	{
		HttpModule = new FHttpModule();
		static_cast<IModuleInterface*>(HttpModule)->StartupModule();
	}

	~FHttpAndWebSocketsFixture()
	{
		// Run the tick loop here (before shutting down HttpModule) so all in-flight requests finish.
		// bQuitRequested may already be true if the loop ran earlier; RunUntilQuitRequested exits immediately in that case.
		TickRunner.RunUntilQuitRequested();

		if (HttpModule != nullptr)
		{
			static_cast<IModuleInterface*>(HttpModule)->ShutdownModule();
			delete HttpModule;
			HttpModule = nullptr;
		}
	}

	FHttpModule* HttpModule = nullptr;
};

TEST_CASE_METHOD(FHttpAndWebSocketsFixture, "WebSockets works correctly when HttpModule is also loaded", WEBSOCKETS_TAG)
{
	// Both a WebSocket echo and an HTTP GET must complete before the test passes.
	auto bHttpDone = MakeShared<bool>(false);
	auto bWsDone = MakeShared<bool>(false);

	auto TryFinish = [this, bHttpDone, bWsDone]()
	{
		if (*bHttpDone && *bWsDone)
		{
			CHECK(HasSucceeded());
			RequestQuit();
		}
	};

	// Fire an HTTP request via FHttpModule while WebSockets is also active.
	TSharedRef<IHttpRequest, ESPMode::ThreadSafe> HttpRequest = HttpModule->CreateRequest();
	HttpRequest->SetURL(FString::Format(TEXT("{0}/methods"), { *Settings.UrlHttpTests() }));
	HttpRequest->SetVerb(TEXT("GET"));
	HttpRequest->OnProcessRequestComplete().BindLambda([bHttpDone, TryFinish](FHttpRequestPtr /* Request */, FHttpResponsePtr /* Response */, bool bProcessedSuccessfully)
	{
		CHECK(bProcessedSuccessfully);
		*bHttpDone = true;
		TryFinish();
	});
	HttpRequest->ProcessRequest();

	TSharedPtr<IWebSocket> WebSocket = WebSocketsModule->CreateWebSocket(FString::Format(TEXT("{0}/echo/"), { *UrlWebSocketsTests() }));

	WebSocket->OnConnected().AddLambda([this, WebSocket](){
		WebSocket->Send(TEXT("hi http+websockets coexistence test"));
	});

	WebSocket->OnMessage().AddLambda([this, WebSocket](const FString& MessageString) {
		CHECK(MessageString == TEXT("hi http+websockets coexistence test"));
		WebSocket->Close();
		SetSucceeded();
	});

	WebSocket->OnClosed().AddLambda([bWsDone, TryFinish](int32 /* StatusCode */, const FString& /* Reason */, bool /* bWasClean */){
		*bWsDone = true;
		TryFinish();
	});

	WebSocket->OnConnectionError().AddLambda([this](const FString& /* Error */){
		CHECK(false);
		RequestQuit();
	});

	WebSocket->Connect();
}

// Fixture that starts FHttpModule first, then FWebSocketsModule after, verifying the two can coexist
// regardless of initialization order.
class FHttpFirstThenWebSocketsFixture
{
public:
	FHttpFirstThenWebSocketsFixture()
		: OldVerbosity(LogWebSockets.GetVerbosity())
	{
		CVarHttpInsecureProtocolEnabled->Set(true);

		// Http starts first — this is the key difference from FHttpAndWebSocketsFixture.
		HttpModule = new FHttpModule();
		static_cast<IModuleInterface*>(HttpModule)->StartupModule();

		// WebSockets starts second.
		WebSocketsModule = new FWebSocketsModule();
		static_cast<IModuleInterface*>(WebSocketsModule)->StartupModule();
	}

	~FHttpFirstThenWebSocketsFixture()
	{
		TickRunner.RunUntilQuitRequested();

		// WebSockets shuts down first.
		static_cast<IModuleInterface*>(WebSocketsModule)->ShutdownModule();
		delete WebSocketsModule;
		WebSocketsModule = nullptr;

		// Http shuts down last.
		static_cast<IModuleInterface*>(HttpModule)->ShutdownModule();
		delete HttpModule;
		HttpModule = nullptr;

		if (OldVerbosity != LogWebSockets.GetVerbosity())
		{
			LogWebSockets.SetVerbosity(OldVerbosity);
		}
	}

	const FString UrlWebSocketsTests() const { return Settings.UrlWebSocketsTests(); }

	void RequestQuit()          { TickRunner.bQuitRequested = true; }
	void SetSucceeded()         { TickRunner.bSucceeded = true; }
	bool HasSucceeded() const   { return TickRunner.bSucceeded; }

	FWebTestTickRunner TickRunner;
	FWebTestSettings Settings;
	FHttpModule* HttpModule = nullptr;
	FWebSocketsModule* WebSocketsModule = nullptr;
	ELogVerbosity::Type OldVerbosity;
};

TEST_CASE_METHOD(FHttpFirstThenWebSocketsFixture, "WebSockets works correctly when HttpModule is loaded first", "Exclude")
{
	// Both a WebSocket echo and an HTTP GET must complete before the test passes.
	auto bHttpDone = MakeShared<bool>(false);
	auto bWsDone = MakeShared<bool>(false);

	auto TryFinish = [this, bHttpDone, bWsDone]()
	{
		if (*bHttpDone && *bWsDone)
		{
			CHECK(HasSucceeded());
			RequestQuit();
		}
	};

	// Fire an HTTP request via FHttpModule while WebSockets is also active.
	TSharedRef<IHttpRequest, ESPMode::ThreadSafe> HttpRequest = HttpModule->CreateRequest();
	HttpRequest->SetURL(FString::Format(TEXT("{0}/methods"), { *Settings.UrlHttpTests() }));
	HttpRequest->SetVerb(TEXT("GET"));
	HttpRequest->OnProcessRequestComplete().BindLambda([bHttpDone, TryFinish](FHttpRequestPtr /* Request */, FHttpResponsePtr /* Response */, bool bProcessedSuccessfully)
	{
		CHECK(bProcessedSuccessfully);
		*bHttpDone = true;
		TryFinish();
	});
	HttpRequest->ProcessRequest();

	TSharedPtr<IWebSocket> WebSocket = WebSocketsModule->CreateWebSocket(FString::Format(TEXT("{0}/echo/"), { *UrlWebSocketsTests() }));

	WebSocket->OnConnected().AddLambda([this, WebSocket](){
		WebSocket->Send(TEXT("hi http-first websockets coexistence test"));
	});

	WebSocket->OnMessage().AddLambda([this, WebSocket](const FString& MessageString) {
		CHECK(MessageString == TEXT("hi http-first websockets coexistence test"));
		WebSocket->Close();
		SetSucceeded();
	});

	WebSocket->OnClosed().AddLambda([bWsDone, TryFinish](int32 /* StatusCode */, const FString& /* Reason */, bool /* bWasClean */){
		*bWsDone = true;
		TryFinish();
	});

	WebSocket->OnConnectionError().AddLambda([this](const FString& /* Error */){
		CHECK(false);
		RequestQuit();
	});

	WebSocket->Connect();
}
