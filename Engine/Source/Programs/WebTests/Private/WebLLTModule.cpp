// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Modules/ModuleManager.h"
#include "TestRunner.h"

#include "TestCommon/Initialization.h"
#include <catch2/catch_test_macros.hpp>

GROUP_AFTER_GLOBAL(Catch::DefaultGroup)
{
	CleanupLocalization();
}

namespace UE::WebTests
{

TArray<FString> GetRequiredModules()
{
	static TArray<FString> Modules({
		FString(TEXT("Sockets"))
	});

	return Modules;
}

void InitializeWebTests()
{
	for (const FString& ModuleName : GetRequiredModules())
	{
		FModuleManager::LoadModulePtr<IModuleInterface>(*ModuleName);
	}
}

void CleanupWebTests()
{
	for (const FString& ModuleName : GetRequiredModules())
	{
		if (IModuleInterface* Module = FModuleManager::Get().GetModule(*ModuleName))
		{
			Module->ShutdownModule();
		}
	}
}

class WebTestsGlobalSetup
{
public:
	WebTestsGlobalSetup()
	{
		FTestDelegates::GetGlobalSetup().BindStatic(&InitializeWebTests);
		FTestDelegates::GetGlobalTeardown().BindStatic(&CleanupWebTests);
	}
} GWebTestsGlobalSetup;

} // UE::WebTests
