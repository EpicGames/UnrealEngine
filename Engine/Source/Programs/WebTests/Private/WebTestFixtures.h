// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/BackgroundableTicker.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformProcess.h"
#include "Misc/CommandLine.h"
#include "Misc/Parse.h"

/**
 *  Shared fixture utilities for WebTests (Http and WebSockets tests)
 *  -----------------------------------------------------------------------
 *
 *  Contains:
 *  - CVarHttpInsecureProtocolEnabled extern (defined in HttpModule.cpp)
 *  - FWebTestSettings: command-line driven server address and verbosity helpers
 *  - FWebTestTickRunner: engine tick simulation with quit-requested loop
 *
 *  -----------------------------------------------------------------------
 */

/** Defined in HttpModule.cpp — allows tests to connect over plain http:// / ws:// */
extern TAutoConsoleVariable<bool> CVarHttpInsecureProtocolEnabled;

/**
 * Parses web test runtime settings from the command line.
 * Http and WebSocket tests may use different ports on the same server host.
 *
 * Command-line parameters:
 *   -web_server_ip=<ip>                    (default: 127.0.0.1)
 *   -web_server_http_port=<port>            (default: 8000)
 *   -web_server_websockets_port=<port>      (default: 8001)
 *   -very_verbose=true                      (default: false) — enables VeryVerbose log level
 */
struct FWebTestSettings
{
	FWebTestSettings()
		: WebServerIp(TEXT("127.0.0.1"))
		, WebServerHttpPort(8000)
		, WebServerWsPort(8001)
		, bVeryVerbose(false)
	{
		FParse::Value(FCommandLine::Get(), TEXT("web_server_ip="), WebServerIp);
		FParse::Value(FCommandLine::Get(), TEXT("web_server_http_port="), WebServerHttpPort);
		FParse::Value(FCommandLine::Get(), TEXT("web_server_websockets_port="), WebServerWsPort);
		FParse::Bool(FCommandLine::Get(), TEXT("very_verbose="), bVeryVerbose);
	}

	/** Base HTTP URL: "http://<ip>:<port>" */
	FString HttpUrlBase() const { return FString::Format(TEXT("http://{0}:{1}"), { *WebServerIp, WebServerHttpPort }); }

	/** URL prefix for the http test routes: "http://<ip>:<port>/webtests/httptests" */
	FString UrlHttpTests() const { return FString::Format(TEXT("{0}/webtests/httptests"), { *HttpUrlBase() }); }

	/** Base WebSocket URL: "ws://<ip>:<wsport>" */
	FString WsUrlBase() const { return FString::Format(TEXT("ws://{0}:{1}"), { *WebServerIp, WebServerWsPort }); }

	/** URL prefix for the websocket test routes: "ws://<ip>:<wsport>/webtests/websocketstests" */
	FString UrlWebSocketsTests() const { return FString::Format(TEXT("{0}/webtests/websocketstests"), { *WsUrlBase() }); }

	FString WebServerIp;
	uint32  WebServerHttpPort;
	uint32  WebServerWsPort;
	bool    bVeryVerbose;
};

/**
 * Helper that provides engine-tick simulation for integration tests that drive async operations
 * (Http requests, WebSocket connections, etc.) from the game thread.
 *
 * Drives FTSBackgroundableTicker and FTSTicker at 60 FPS until RequestQuit() is called.
 * Intended to be used as a member variable in test fixtures, not as a base class.
 */
class FWebTestTickRunner
{
public:
	void SimulateEngineTick()
	{
		FTSBackgroundableTicker::GetCoreTicker().Tick(TickFrequency);
		FTSTicker::GetCoreTicker().Tick(TickFrequency);
		FPlatformProcess::Sleep(TickFrequency);
	}

	void RunUntilQuitRequested()
	{
		while (!bQuitRequested)
		{
			SimulateEngineTick();
		}
	}

	float TickFrequency  = 1.0f / 60; /*60 FPS*/
	bool  bQuitRequested = false;
	bool  bSucceeded     = false;
};
