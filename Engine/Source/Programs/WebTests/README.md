WebTests
--------------------

This is a collection of tests for web features like http, websocket, ssl etc.

# Running the Tests

## Start the test server (NOTE: for TEST PURPOSE ONLY, not production ready)

Two server options are available. Both listen on HTTP port 8000 and WebSocket port 8001.

### Option A: C++ server (recommended — no Python required)

Windows:
	In ../WebTestsServerCpp folder:
		> runserver.bat

Mac/Linux:
	In ../WebTestsServerCpp folder:
		> sh runserver.sh

### Option B: Python server (Django Channels / Daphne)

Windows:
	In ../WebTestsServerPy folder:
		> runserver.bat

Mac/Linux:
	In ../WebTestsServerPy folder:
		> sh runserver.sh

## Run the tests from VS:
	Set `WebTests` as the startup project and set Solution Configuration to `Development`.
	If running tests on other devices, pass in the ip as command line args, after extra args AT THE END, like: "--extra-args --web_server_ip=your.pc.ip.address --log"
	Compile and debug

## Compile and run the tests from command line:
To compile:
	.\Engine\Build\BatchFiles\RunUBT.bat WebTests Win64 Development -Progress

To compile for any other platform, replace "Win64" in the param

It uses Catch2 test framework, so in order to run with specific case or tag, pass the filter as first param, for example:
	.\Engine\Binaries\Win64\WebTests\WebTests.exe "Get domain of url can work well"
	.\Engine\Binaries\Win64\WebTests\WebTests.exe "[WebSockets]"
	.\Engine\Binaries\Win64\WebTests\WebTests.exe "[HTTP]"

# Adding new test cases in WebTestsServerPy (Python):
	Add/change the code in ./WebTestsServerPy, most likely in httptests/urls.py and httptests/views.py, and save. Code will be reloaded if the web server is running.
	For more info about how to code in Django, see https://docs.djangoproject.com/en/

# Adding new test cases in WebTestsServerCpp (C++):

After editing either file below, rebuild WebTestsServerCpp and restart the server.

## New WebSocket route (WebTestsServerCpp/Private/WebTestsWebSocketServer.cpp)

Add a new `else if` branch in the `OnMessage` handler inside `StartWebSocketServer()`:

	else if (Path.EndsWith(TEXT("/your_route/")))
	{
	    // echo with modification:
	    Conn->SendText(FString::Printf(TEXT("pong: %s"), *Msg));
	    // or close the connection:
	    // Conn->Close(1000);
	}

The path matches the URL the client connects to, e.g. ws://localhost:8001/webtests/websocketstests/your_route/

## New HTTP route (WebTestsServerCpp/Private/WebTestsHttpServer.cpp)

1. Write a handler function above StartHttpServer():

	static bool HandleYourRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
	{
	    LogRequest(Request, 200);
	    OnComplete(JsonResponse(TEXT("{\"ok\":true}")));
	    return true;
	}

2. Register it in StartHttpServer() with Router->BindRoute:

	Router->BindRoute(FHttpPath(TEXT("/webtests/httptests/your_route")),
	    EVerb::VERB_GET,
	    FHttpRequestHandler::CreateStatic(&HandleYourRoute));

Use :param_name segments for path parameters (e.g. /your_route/:id/); read them with Request.PathParams.Find(TEXT("id")).
Query parameters are in Request.QueryParams.
