// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class WebTests : TestModuleRules
{
	static WebTests()
	{
		if (InTestMode)
		{
			TestMetadata = new Metadata();
			TestMetadata.TestName = "Web";
			TestMetadata.TestShortName = "Web";
			TestMetadata.ReportType = "testdata";
			TestMetadata.SupportedPlatforms.Add(UnrealTargetPlatform.Linux);
			TestMetadata.SupportedPlatforms.Add(UnrealTargetPlatform.Mac);
			TestMetadata.SupportedPlatforms.Add(UnrealTargetPlatform.Android);
			TestMetadata.SupportedPlatforms.Add(UnrealTargetPlatform.IOS);

			TestMetadata.HasAfterSteps = false;
			TestMetadata.PlatformCompilationExtraArgs.Add(UnrealTargetPlatform.Win64, "-allmodules");
			TestMetadata.PlatformCompilationExtraArgs.Add(UnrealTargetPlatform.Linux, "-allmodules");
			TestMetadata.PlatformCompilationExtraArgs.Add(UnrealTargetPlatform.LinuxArm64, "-allmodules");
			TestMetadata.PlatformCompilationExtraArgs.Add(UnrealTargetPlatform.Mac, "-allmodules");
			TestMetadata.PlatformCompilationExtraArgs.Add(UnrealTargetPlatform.IOS, "-allmodules");
			TestMetadata.PlatformCompilationExtraArgs.Add(UnrealTargetPlatform.TVOS, "-allmodules");
			TestMetadata.PlatformCompilationExtraArgs.Add(UnrealTargetPlatform.Android, "-allmodules  -architectures=arm64");
			TestMetadata.PlatformCompilationExtraArgs.Add(UnrealTargetPlatform.VisionOS, "-allmodules");
		}
	}

	public WebTests(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"ApplicationCore",
				"Core",
				"EventLoop",
				"HTTP",
				"SSL",
				"HTTPServer",
				"Sockets",
				"WebSockets",
				"Json"
			});
	}
}


