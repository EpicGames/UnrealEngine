// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatforms(UnrealPlatformClass.All)]
public class WebTestsTarget : TestTargetRules
{
	public WebTestsTarget(TargetInfo Target) : base(Target)
	{
		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;
		bCompileAgainstCoreUObject = true;
		bCompileAgainstApplicationCore = true;
	}
}
